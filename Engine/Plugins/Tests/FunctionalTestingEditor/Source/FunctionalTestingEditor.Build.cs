// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class FunctionalTestingEditor : ModuleRules
{
    public FunctionalTestingEditor(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange
        (
            new string[] {
				"Core",
                "InputCore",
                "CoreUObject",
                "SlateCore",
                "Slate",
                
                "Engine",
                "AssetRegistry"
			}
        );
        
        PrivateDependencyModuleNames.AddRange(
             new string[] {
				"AssetDefinition",
				"EditorFramework",
                "UnrealEd",
                "LevelEditor",
                "SessionFrontend",
                "FunctionalTesting",
                "PlacementMode",
                "WorkspaceMenuStructure",
                "ScreenShotComparisonTools",
                "ToolMenus",
            }
         );
	}
}
