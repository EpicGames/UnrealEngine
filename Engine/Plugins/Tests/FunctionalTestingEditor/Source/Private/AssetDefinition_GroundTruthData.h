// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "GroundTruthData.h"
#include "AssetDefinition_GroundTruthData.generated.h"

UCLASS()
class UAssetDefinition_GroundTruthData : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_GroundTruthData", "Ground Truth Data"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UGroundTruthData::StaticClass(); }
	virtual FLinearColor GetAssetColor() const override { return FColor(255, 196, 128); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Misc, NSLOCTEXT("AssetDefinition", "AssetDefinition_GroundTruthDataSubMenu", "Other"), ECategoryMenuType::Section)
		};
		return Categories;
	}
	// UAssetDefinition End
};
