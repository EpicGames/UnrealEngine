// Copyright Epic Games, Inc. All Rights Reserved.

#include "FunctionalTestingEditorModule.h"
#include "Engine/Level.h"
#include "Framework/Application/SlateApplication.h"
#include "LevelEditor.h"
#include "ISessionFrontendModule.h"
#include "IPlacementModeModule.h"
#include "ScreenshotFunctionalTest.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ToolMenu.h"
#include "WorkspaceMenuStructure.h"
#include "ToolMenuSection.h"
#include "WorkspaceMenuStructureModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "EditorAutomation"

void OpenMapAndFocusActor(const TArray<FString>& Args)
{
	if ( Args.Num() != 2 )
	{
		UE_LOGF(LogConsoleResponse, Display, "Automate.OpenMapAndFocusActor failed, the number of arguments is wrong.  Automate.OpenMapAndFocusActor MapObjectPath ActorName\n");
		return;
	}

	FString AssetPath(Args[0]);
	FString ActorName(Args[1]);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	FAssetData MapAssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPath));

	if ( MapAssetData.IsValid() )
	{
		bool bIsWorlAlreadyOpened = false;
		if ( UWorld* EditorWorld = GEditor->GetEditorWorldContext().World() )
		{
			if ( FAssetData(EditorWorld).PackageName == MapAssetData.PackageName )
			{
				bIsWorlAlreadyOpened = true;
			}
		}

		if ( !bIsWorlAlreadyOpened )
		{
			UObject* ObjectToEdit = MapAssetData.GetAsset();
			if ( ObjectToEdit )
			{
				GEditor->EditObject(ObjectToEdit);
			}
		}
	}

	if ( UWorld* EditorWorld = GEditor->GetEditorWorldContext().World() )
	{
		AActor* ActorToFocus = nullptr;
		for ( int32 LevelIndex=0; LevelIndex < EditorWorld->GetNumLevels(); LevelIndex++ )
		{
			ULevel* Level = EditorWorld->GetLevel(LevelIndex);
			ActorToFocus = static_cast<AActor*>( FindObjectWithOuter(Level, AActor::StaticClass(), *Args[1]) );
			if ( ActorToFocus )
			{
				break;
			}
		}

		if ( ActorToFocus )
		{
			GEditor->SelectNone(/*bNoteSelectionChange=*/ false, false, false);
			GEditor->SelectActor(ActorToFocus, true, true);
			GEditor->NoteSelectionChange();

			const bool bActiveViewportOnly = false;
			GEditor->MoveViewportCamerasToActor(*ActorToFocus, bActiveViewportOnly);

			FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
			TSharedPtr<SDockTab> LevelEditorTab = LevelEditorModule.GetLevelEditorInstanceTab().Pin();
			if ( LevelEditorTab.IsValid() )
			{
				LevelEditorTab->DrawAttention();
			}
		}
	}
}

FAutoConsoleCommand OpenMapAndFocusActorCmd(
	TEXT("Automate.OpenMapAndFocusActor"),
	TEXT("Opens a map and focuses a particular actor by name."),
	FConsoleCommandWithArgsDelegate::CreateStatic(OpenMapAndFocusActor)
);

class FFunctionalTestingEditorModule : public IFunctionalTestingEditorModule
{
	void StartupModule()
	{
		if (FSlateApplication::IsInitialized())
		{
			// Add Automation Area to the Tools Menu
			UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Tools");
			FToolMenuSection& Section = Menu->AddSection("Automation", LOCTEXT("AutomationHeading", "Automation"));
			Section.AddMenuEntry( 
				"TestAutomation",
				LOCTEXT("AutomationLabel", "Test Automation"),
				LOCTEXT("Tooltip", "Launch the Testing Automation Frontend."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "AutomationTools.TestAutomation"),
				FUIAction(FExecuteAction::CreateStatic(&FFunctionalTestingEditorModule::OnShowAutomationFrontend))
			);
		}

		FModuleManager::Get().OnModulesChanged().AddRaw(this, &FFunctionalTestingEditorModule::OnModulesChanged);

		if ( IPlacementModeModule::IsAvailable() )
		{
			OnModulesChanged("PlacementMode", EModuleChangeReason::ModuleLoaded);
		}
	}

	void ShutdownModule()
	{
		if ( FModuleManager::Get().IsModuleLoaded("LevelEditor") )
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
			LevelEditorModule.GetMenuExtensibilityManager()->RemoveExtender(Extender);
		}

		FModuleManager::Get().OnModulesChanged().RemoveAll(this);

		if ( IPlacementModeModule::IsAvailable() )
		{
			IPlacementModeModule::Get().UnregisterPlacementCategory("Testing");
		}
	}

	void PopulateAutomationTools(FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.BeginSection("AutomationTools", LOCTEXT("AutomationTools", "Automation Tools"));
		const bool bAutoAndOrphanedMenus = false;
		FGlobalTabmanager::Get()->PopulateTabSpawnerMenu(MenuBuilder, WorkspaceMenu::GetMenuStructure().GetAutomationToolsCategory(), bAutoAndOrphanedMenus);
		MenuBuilder.EndSection();
	}

	static void OnShowAutomationFrontend()
	{
		ISessionFrontendModule& SessionFrontend = FModuleManager::LoadModuleChecked<ISessionFrontendModule>("SessionFrontend");
		SessionFrontend.InvokeSessionFrontend(FName("AutomationPanel"));
	}

	void OnModulesChanged(FName Module, EModuleChangeReason Reason)
	{
		if ( Module == TEXT("PlacementMode") && Reason == EModuleChangeReason::ModuleLoaded )
		{
			FPlacementCategoryInfo Info(
				LOCTEXT("FunctionalTestingCategoryName", "Testing"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.Testing"),
				"Testing",
				TEXT("PMTesting"),
				25
				);

			IPlacementModeModule::Get().RegisterPlacementCategory(Info);
			IPlacementModeModule::Get().RegisterPlaceableItem(Info.UniqueHandle, MakeShareable(new FPlaceableItem(nullptr, FAssetData(AFunctionalTest::StaticClass()))));
			IPlacementModeModule::Get().RegisterPlaceableItem(Info.UniqueHandle, MakeShareable(new FPlaceableItem(nullptr, FAssetData(AScreenshotFunctionalTest::StaticClass()))));
		}
	}

private:
	TSharedPtr<FExtender> Extender;
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FFunctionalTestingEditorModule, FunctionalTestingEditor);
