// Copyright Epic Games, Inc. All Rights Reserved.

#include "GroundTruthDataFactory.h"
#include "GroundTruthData.h"
#include "AssetTypeCategories.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GroundTruthDataFactory)

#define LOCTEXT_NAMESPACE "UGroundTruthDataFactory"

UGroundTruthDataFactory::UGroundTruthDataFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = false;
	bEditorImport = false;
	SupportedClass = UGroundTruthData::StaticClass();
}

UObject* UGroundTruthDataFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* Context, FFeedbackContext* Warn)
{
	UGroundTruthData* GroundTruthData = NewObject<UGroundTruthData>(InParent, SupportedClass, InName, InFlags | RF_Transactional);
	return GroundTruthData;
}

FText UGroundTruthDataFactory::GetDisplayName() const
{
	return LOCTEXT("MenuEntry", "Ground Truth Data");
}

#undef LOCTEXT_NAMESPACE

