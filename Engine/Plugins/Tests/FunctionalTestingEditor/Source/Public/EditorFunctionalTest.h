// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenshotFunctionalTest.h"
#include "EditorFunctionalTest.generated.h"

/** Editor-only version of functional test actor, tests that call editor-only blueprints must inherit from this class */
UCLASS(Blueprintable, MinimalAPI)
class AEditorFunctionalTest : public AFunctionalTest
{
	GENERATED_BODY()

	virtual bool IsEditorOnly() const override
	{
		return true;
	}

	virtual bool IsEditorOnlyLoadedInPIE() const override
	{
		return true;
	}
};

/** Editor-only version of screenshot functional test */
UCLASS(Blueprintable, MinimalAPI)
class AEditorScreenshotFunctionalTest : public AScreenshotFunctionalTest
{
	GENERATED_BODY()

	virtual bool IsEditorOnly() const override
	{
		return true;
	}

	virtual bool IsEditorOnlyLoadedInPIE() const override
	{
		return true;
	}
};
