// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"


/**
 * A module for adding automation exposure in the editor
 */
class IFunctionalTestingEditorModule : public IModuleInterface
{
public:
	/**
	* Called right after the module DLL has been loaded and the module object has been created
	*/
	virtual void StartupModule() = 0;

	/**
	* Called before the module is unloaded, right before the module object is destroyed.
	*/
	virtual void ShutdownModule() = 0;
};
